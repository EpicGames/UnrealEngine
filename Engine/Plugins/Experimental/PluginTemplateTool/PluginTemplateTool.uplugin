{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Plugin Template Tool",
	"Description" : "Editor plugin for managing plugin template content folders.",
	"Category" : "Editor",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "http://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : true,
	"Installed" : false,
	
	"Modules" :
	[
		{
			"Name": "PluginTemplateTool",
			"Type": "Editor",
			"LoadingPhase": "PreDefault"
		}
	],
	"Plugins": [
		{
			"Name": "PluginBrowser",
			"Enabled": true
		},
	]
}