// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class PluginTemplateTool : ModuleRules
{
	public PluginTemplateTool(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.AddRange(
			new string[] {
			}
		);

		PublicDependencyModuleNames.AddRange(
			new string[] { 
				"Core",
				"CoreUObject",
				"InputCore",
				"Engine",
				"Slate",
				"SlateCore",
			}
		);
		
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"EditorFramework",
				"PluginBrowser",
				"ToolWidgets",
				"EditorWidgets",
				"WorkspaceMenuStructure",
			}
		);
	}
}
