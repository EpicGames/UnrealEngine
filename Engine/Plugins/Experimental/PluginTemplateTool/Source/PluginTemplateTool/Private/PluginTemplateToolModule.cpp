// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleInterface.h"
#include "SPluginTemplateBrowser.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#define LOCTEXT_NAMESPACE "PluginTemplateTool"

class FPluginTemplateToolModule : public IModuleInterface
{
public:
	// IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	// End of IModuleInterface interface

private:
	static const FName PluginTemplateToolTabName;
	TSharedRef<SDockTab> HandleSpawnPluginTemplatesTab(const FSpawnTabArgs& Args);
};

const FName FPluginTemplateToolModule::PluginTemplateToolTabName = TEXT("PluginTemplateTool");

///////////////////////////////////////////

IMPLEMENT_MODULE(FPluginTemplateToolModule, PluginTemplateTool);


void FPluginTemplateToolModule::StartupModule()
{
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(PluginTemplateToolTabName, FOnSpawnTab::CreateRaw(this, &FPluginTemplateToolModule::HandleSpawnPluginTemplatesTab))
		.SetDisplayName(LOCTEXT("PluginTemplateTitle", "Plugin Template Tool"))
		.SetTooltipText(LOCTEXT("PluginTemplateTooltip", "Open Plugin Template Tool window."))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetToolsCategory())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Audit"));
	FGlobalTabmanager::Get()->RegisterDefaultTabWindowSize(PluginTemplateToolTabName, FVector2D(1080, 600));
}

void FPluginTemplateToolModule::ShutdownModule()
{
	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(PluginTemplateToolTabName);
}

TSharedRef<SDockTab> FPluginTemplateToolModule::HandleSpawnPluginTemplatesTab(const FSpawnTabArgs& Args)
{
	return SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		[
			SNew(SPluginTemplateBrowser)
		];
}

#undef LOCTEXT_NAMESPACE
