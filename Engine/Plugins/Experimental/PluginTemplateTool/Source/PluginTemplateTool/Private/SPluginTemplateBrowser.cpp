// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPluginTemplateBrowser.h"

#include "Misc/MessageDialog.h"
#include "Misc/PackageName.h"
#include "IPluginBrowser.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableViewBase.h"

#define LOCTEXT_NAMESPACE "PluginTemplateBrowser"

void SPluginTemplateBrowser::Construct(const FArguments& InArgs)
{
	const TArray<TSharedRef<FPluginTemplateDescription>>& PluginTemplateDescriptions = IPluginBrowser::Get().GetAddedPluginTemplates();
	for (const auto& Item : PluginTemplateDescriptions)
	{
		TSharedRef<FPluginTemplateListItem> NewItem(new FPluginTemplateListItem(Item->Name, Item->OnDiskPath));
		PluginTemplateListItems.Add(NewItem);
	}

	ChildSlot
		.Padding(FMargin(8))
	[
		SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SPluginTemplateListView)
					.SelectionMode(ESelectionMode::Single)
					.ListItemsSource(&PluginTemplateListItems)
					.OnGenerateRow(this, &SPluginTemplateBrowser::OnGenerateWidgetForTemplateListView)
					.HeaderRow
					(
						SNew(SHeaderRow)
						+ SHeaderRow::Column("TemplateNames")
						.DefaultLabel(NSLOCTEXT("PluginTemplateBrowser", "TemplateName", "Template"))
						.FillWidth(0.7f)
						+ SHeaderRow::Column("TemplateActions")
						.DefaultLabel(NSLOCTEXT("PluginTemplateBrowser", "TemplateActions", "Actions"))
						.FillWidth(0.36f)
						+ SHeaderRow::Column("TemplatePaths")
						.DefaultLabel(NSLOCTEXT("PluginTemplateBrowser", "TemplatePath", "Path"))
					)
			]
	];
}

TSharedRef<ITableRow> SPluginTemplateBrowser::OnGenerateWidgetForTemplateListView(TSharedPtr<FPluginTemplateListItem> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	class STemplateItemWidget : public SMultiColumnTableRow<TSharedPtr<FPluginTemplateListItem>>
	{
	public:
		SLATE_BEGIN_ARGS(STemplateItemWidget) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, TSharedPtr<FPluginTemplateListItem> InListItem)
		{
			Item = InListItem;

			SMultiColumnTableRow<TSharedPtr<FPluginTemplateListItem>>::Construct(FSuperRowType::FArguments(), InOwnerTable);
		}

		TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName)
		{
			if (ColumnName == "TemplateNames")
			{
				return
					SNew(STextBlock)
					.Text(Item->TemplateName);
			}
			else if (ColumnName == "TemplateActions")
			{
				return
					SNew(SHorizontalBox)

					// Mount button
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2.0f, 0.0f)
					[
						SNew(SButton)
						.Visibility(Item.ToSharedRef(), &FPluginTemplateListItem::GetVisibilityBasedOnUnmountedState)
						.Text(LOCTEXT("Mount", "Mount"))
						.OnClicked(Item.ToSharedRef(), &FPluginTemplateListItem::OnMountClicked)
					]

					// Unmount button
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2.0f, 0.0f)
					[
						SNew(SButton)
						.Visibility(Item.ToSharedRef(), &FPluginTemplateListItem::GetVisibilityBasedOnMountedState)
						.Text(LOCTEXT("Unmount", "Unmount"))
						.OnClicked(Item.ToSharedRef(), &FPluginTemplateListItem::OnUnmountClicked)
					];
			}
			else if (ColumnName == "TemplatePaths")
			{
				return
					SNew(STextBlock)
					.Text(FText::FromString(Item->OnDiskPath));
			}
			else
			{
				return SNew(STextBlock).Text(LOCTEXT("UnknownColumn", "Unknown Column"));
			}

		}

		TSharedPtr<FPluginTemplateListItem> Item;
	};

	return SNew(STemplateItemWidget, OwnerTable, InItem);
}

SPluginTemplateBrowser::FPluginTemplateListItem::FPluginTemplateListItem(FText InTemplateName, FString InOnDiskPath)
	: TemplateName(InTemplateName)
	, OnDiskPath(InOnDiskPath)
{
	bIsMounted = FPackageName::MountPointExists(GetRootPath());
}

FString SPluginTemplateBrowser::FPluginTemplateListItem::GetRootPath() const
{
	return TEXT("/") + FPaths::GetBaseFilename(OnDiskPath) + TEXT("/");
}

FString SPluginTemplateBrowser::FPluginTemplateListItem::GetContentPath() const
{
	return OnDiskPath / TEXT("Content/");
}

FReply SPluginTemplateBrowser::FPluginTemplateListItem::OnMountClicked()
{
	const FString& RootPath = GetRootPath();
	const FString& ContentPath = GetContentPath();

	FText FailReason;
	if (ContentPath[0] == TEXT('/'))
	{
		FailReason = LOCTEXT("MountPluginTemplate_InvalidContentPath", "Invalid ContentPath, should not start with '/'! Example: '../../../ProjectName/Content/'");
	}

	if (RootPath[0] != TEXT('/'))
	{
		FailReason = LOCTEXT("MountPluginTemplate_InvalidRootPath", "Invalid RootPath, should start with a '/'! Example: '/Game/'");
	}

	if (!FPaths::DirectoryExists(ContentPath))
	{
		FailReason = FText::Format(LOCTEXT("MountPluginTemplate_ContentNotExists", "Content Path: {0} does not exist"), FText::FromString(FPaths::ConvertRelativePathToFull(ContentPath)));
	}

	if (FailReason.IsEmpty())
	{
		FPackageName::RegisterMountPoint(RootPath, ContentPath);
		bIsMounted = true;
	}
	else
	{
		FMessageDialog::Open(EAppMsgType::Ok, FailReason, LOCTEXT("UnableToMountPluginTemplate", "Unable to mount plugin template"));
	}

	return FReply::Handled();
}

FReply SPluginTemplateBrowser::FPluginTemplateListItem::OnUnmountClicked()
{
	check(FPackageName::MountPointExists(GetRootPath()));

	FPackageName::UnRegisterMountPoint(GetRootPath(), GetContentPath());
	bIsMounted = false;

	return FReply::Handled();
}

EVisibility SPluginTemplateBrowser::FPluginTemplateListItem::GetVisibilityBasedOnMountedState() const
{
	return bIsMounted ? EVisibility::Visible : EVisibility::Hidden;
}

EVisibility SPluginTemplateBrowser::FPluginTemplateListItem::GetVisibilityBasedOnUnmountedState() const
{
	return !bIsMounted ? EVisibility::Visible : EVisibility::Hidden;
}

#undef LOCTEXT_NAMESPACE