// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class ITableRow;
class STableViewBase;

template <typename ItemType> class SListView;

//////////////////////////////////////////////////////////////////////////
// SPluginTemplateBrowser

class SPluginTemplateBrowser : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPluginTemplateBrowser) {}
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);

private:
	/** An item in the plugin template list */
	struct FPluginTemplateListItem
	{
		FPluginTemplateListItem(FText InTemplateName, FString InOnDiskPath);

		FText TemplateName;
		FString OnDiskPath;
		bool bIsMounted =false;

		FString GetRootPath() const;
		FString GetContentPath() const;

		FReply OnMountClicked();
		FReply OnUnmountClicked();

		/** Returns Visible for templates that are currently mounted, otherwise Hidden */
		EVisibility GetVisibilityBasedOnMountedState() const;

		/** Returns Hidden for a template that are currently mounted, otherwise Visible */
		EVisibility GetVisibilityBasedOnUnmountedState() const;
	};

	TSharedRef<ITableRow> OnGenerateWidgetForTemplateListView(TSharedPtr<FPluginTemplateListItem> InItem, const TSharedRef<STableViewBase>& OwnerTable);

private:
	typedef SListView<TSharedPtr<FPluginTemplateListItem>> SPluginTemplateListView;

	/** List items for the plugin template list */
	TArray<TSharedPtr<FPluginTemplateListItem>> PluginTemplateListItems;

	/** List of all known plugin templates */
	TSharedPtr<SPluginTemplateListView> PluginTemplateListView;
};